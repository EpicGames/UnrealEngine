{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "UAF Shared Assets",
	"Description": "UAF Default Assets that interact with multiple plugins",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
	],
	"Plugins": [
		{
			"Name": "Workspace",
			"Enabled": true
		},
		{
			"Name": "Chooser",
			"Enabled": true
		},
		{
			"Name": "EvaluationNotifies",
			"Enabled": true
		},
		{
			"Name": "PoseSearch",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true		
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimNode",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true		
		},		
		{
			"Name": "UAFStateTree",
			"Enabled": true		
		}
	]
}
